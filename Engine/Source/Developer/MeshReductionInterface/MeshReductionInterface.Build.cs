// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class MeshReductionInterface : ModuleRules
{
	public MeshReductionInterface(ReadOnlyTargetRules Target) : base(Target)
	{		
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("CoreUObject");
		PrivateDependencyModuleNames.Add("Engine");
    }
}
