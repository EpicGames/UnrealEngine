// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshReductionManagerModule.h"
#include "IMeshReductionInterfaces.h"
#include "CoreGlobals.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"

#include "Engine/MeshSimplificationSettings.h"
#include "Engine/SkeletalMeshSimplificationSettings.h"
#include "Engine/ProxyLODMeshSimplificationSettings.h"

DEFINE_LOG_CATEGORY_STATIC(LogMeshReduction, Verbose, All);

IMPLEMENT_MODULE(FMeshReductionManagerModule, MeshReductionInterface);

FMeshReductionManagerModule::FMeshReductionManagerModule()
	: StaticMeshReduction(nullptr)
	, SkeletalMeshReduction(nullptr)
	, MeshMerging(nullptr)
	, DistributedMeshMerging(nullptr)
{
}

void FMeshReductionManagerModule::StartupModule()
{
	checkf(StaticMeshReduction   == nullptr, TEXT("Static Reduction instance should be null during startup"));
	checkf(SkeletalMeshReduction == nullptr, TEXT("Skeletal Reduction instance should be null during startup"));
	checkf(MeshMerging           == nullptr, TEXT("Mesh Merging instance should be null during startup"));

	// Get settings CDOs
	UMeshSimplificationSettings* MeshSimplificationSettings_CDO = UMeshSimplificationSettings::StaticClass()->GetDefaultObject<UMeshSimplificationSettings>();
	USkeletalMeshSimplificationSettings* SkeletalMeshSimplificationSettings_CDO = USkeletalMeshSimplificationSettings::StaticClass()->GetDefaultObject<USkeletalMeshSimplificationSettings>();
	UProxyLODMeshSimplificationSettings* ProxyLODMeshSimplificationSettings_CDO = UProxyLODMeshSimplificationSettings::StaticClass()->GetDefaultObject<UProxyLODMeshSimplificationSettings>();

	// Update module names from config entries
	{
		FString MeshReductionModuleName;
		if (GConfig->GetString(TEXT("/Script/Engine.MeshSimplificationSettings"), TEXT("r.MeshReductionModule"), MeshReductionModuleName, GEngineIni))
		{
			MeshSimplificationSettings_CDO->SetMeshReductionModuleName(*MeshReductionModuleName);
		}
		FString SkeletalMeshReductionModuleName;
		if (GConfig->GetString(TEXT("/Script/Engine.SkeletalMeshSimplificationSettings"), TEXT("r.SkeletalMeshReductionModule"), SkeletalMeshReductionModuleName, GEngineIni))
		{
			SkeletalMeshSimplificationSettings_CDO->SetSkeletalMeshReductionModuleName(*SkeletalMeshReductionModuleName);
		}
		FString HLODMeshReductionModuleName;
		if (GConfig->GetString(TEXT("/Script/Engine.ProxyLODMeshSimplificationSettings"), TEXT("r.ProxyLODMeshReductionModule"), HLODMeshReductionModuleName, GEngineIni))
		{
			ProxyLODMeshSimplificationSettings_CDO->SetProxyLODMeshReductionModuleName(*HLODMeshReductionModuleName);
		}
	}

	// Get configured module names.
	FName MeshReductionModuleName = MeshSimplificationSettings_CDO->MeshReductionModuleName;
	if (!FModuleManager::Get().ModuleExists(*MeshReductionModuleName.ToString()))
	{
		UE_LOGF(LogMeshReduction, Display, "Mesh reduction module (r.MeshReductionModule) set to \"%ls\" which doesn't exist.", *MeshReductionModuleName.ToString());
	}

	FName SkeletalMeshReductionModuleName = SkeletalMeshSimplificationSettings_CDO->SkeletalMeshReductionModuleName;
	if (!FModuleManager::Get().ModuleExists(*SkeletalMeshReductionModuleName.ToString()))
	{
		UE_LOGF(LogMeshReduction, Display, "Skeletal mesh reduction module (r.SkeletalMeshReductionModule) set to \"%ls\" which doesn't exist.", *SkeletalMeshReductionModuleName.ToString());
	}

	FName HLODMeshReductionModuleName = ProxyLODMeshSimplificationSettings_CDO->ProxyLODMeshReductionModuleName;
	if (!FModuleManager::Get().ModuleExists(*HLODMeshReductionModuleName.ToString()))
	{
		UE_LOGF(LogMeshReduction, Display, "HLOD mesh reduction module (r.ProxyLODMeshReductionModule) set to \"%ls\" which doesn't exist.", *HLODMeshReductionModuleName.ToString());
	}

	// Retrieve reduction interfaces 
	TArray<FName> ModuleNames;
	FModuleManager::Get().FindModules(TEXT("*MeshReduction"), ModuleNames);
	for (FName ModuleName : ModuleNames)
	{
		FModuleManager::Get().LoadModule(ModuleName);
	}

	if (FModuleManager::Get().ModuleExists(TEXT("SimplygonSwarm")))
	{
		FModuleManager::Get().LoadModule("SimplygonSwarm");
	}
	
	TArray<IMeshReductionModule*> MeshReductionModules = IModularFeatures::Get().GetModularFeatureImplementations<IMeshReductionModule>(IMeshReductionModule::GetModularFeatureName());
	
	// Actual module names that will be used.
	FName StaticMeshModuleName;
	FName SkeletalMeshModuleName;
	FName MeshMergingModuleName;
	FName DistributedMeshMergingModuleName;

	for (IMeshReductionModule* Module : MeshReductionModules)
	{
		// Is this a requested module?
		const FName ModuleName(Module->GetName());
		const bool bIsRequestedMeshReductionModule         = ModuleName == MeshReductionModuleName;
		const bool bIsRequestedSkeletalMeshReductionModule = ModuleName == SkeletalMeshReductionModuleName;
		const bool bIsRequestedProxyLODReductionModule     = ModuleName == HLODMeshReductionModuleName;	
	

		// Look for MeshReduction interface
		IMeshReduction* StaticMeshReductionInterface = Module->GetStaticMeshReductionInterface();
		if (StaticMeshReductionInterface)
		{
			if ( bIsRequestedMeshReductionModule || StaticMeshReduction == nullptr )
			{
				StaticMeshReduction  = StaticMeshReductionInterface;
				StaticMeshModuleName = ModuleName;
			}
		}

		// Look for Skeletal MeshReduction interface
		IMeshReduction* SkeletalMeshReductionInterface = Module->GetSkeletalMeshReductionInterface();
		if (SkeletalMeshReductionInterface)
		{
			if ( bIsRequestedSkeletalMeshReductionModule || SkeletalMeshReduction == nullptr )
			{
				SkeletalMeshReduction  = SkeletalMeshReductionInterface;
				SkeletalMeshModuleName = ModuleName;
			}
		}

		// Look for MeshMerging interface
		IMeshMerging* MeshMergingInterface = Module->GetMeshMergingInterface();
		if (MeshMergingInterface)
		{
			if ( bIsRequestedProxyLODReductionModule || MeshMerging == nullptr )
			{
				MeshMerging           = MeshMergingInterface;
				MeshMergingModuleName = ModuleName;
			}
		}

		// Look for Distributed MeshMerging interface
		IMeshMerging* DistributedMeshMergingInterface = Module->GetDistributedMeshMergingInterface();
		if (DistributedMeshMergingInterface)
		{
			if ( bIsRequestedMeshReductionModule || DistributedMeshMerging == nullptr )
			{
				DistributedMeshMerging           = DistributedMeshMergingInterface;
				DistributedMeshMergingModuleName = ModuleName;
			}
		}
	}

	MeshSimplificationSettings_CDO->SetMeshReductionModuleName(StaticMeshModuleName);
	SkeletalMeshSimplificationSettings_CDO->SetSkeletalMeshReductionModuleName(SkeletalMeshModuleName);
	ProxyLODMeshSimplificationSettings_CDO->SetProxyLODMeshReductionModuleName(MeshMergingModuleName);

	if (!StaticMeshReduction)
	{
		UE_LOGF(LogMeshReduction, Display, "No automatic static mesh reduction module available");
	}
	else
	{
		UE_LOGF(LogMeshReduction, Display, "Using %ls for automatic static mesh reduction", *StaticMeshModuleName.ToString());
	}

	if (!SkeletalMeshReduction)
	{
		UE_LOGF(LogMeshReduction, Display, "No automatic skeletal mesh reduction module available");
	}
	else
	{
		UE_LOGF(LogMeshReduction, Display, "Using %ls for automatic skeletal mesh reduction", *SkeletalMeshReductionModuleName.ToString());
	}

	if (!MeshMerging)
	{
		UE_LOGF(LogMeshReduction, Display, "No automatic mesh merging module available");
	}
	else
	{
		UE_LOGF(LogMeshReduction, Display, "Using %ls for automatic mesh merging", *MeshMergingModuleName.ToString());
	}


	if (!DistributedMeshMerging)
	{
		UE_LOGF(LogMeshReduction, Display, "No distributed automatic mesh merging module available");
	}
	else
	{
		UE_LOGF(LogMeshReduction, Display, "Using %ls for distributed automatic mesh merging", *DistributedMeshMergingModuleName.ToString());
	}

	bConfigurationCompleted = true;
	ConfigurationCompletedDelegate.Broadcast();
	ConfigurationCompletedDelegate.Clear();
}

void FMeshReductionManagerModule::ShutdownModule()
{
	StaticMeshReduction = SkeletalMeshReduction = nullptr;
	MeshMerging = DistributedMeshMerging = nullptr;
}

IMeshReduction* FMeshReductionManagerModule::GetStaticMeshReductionInterface() const
{
	return StaticMeshReduction;
}

IMeshReduction* FMeshReductionManagerModule::GetSkeletalMeshReductionInterface() const
{
	return SkeletalMeshReduction;
}

IMeshMerging* FMeshReductionManagerModule::GetMeshMergingInterface() const
{
	return MeshMerging;
}

IMeshMerging* FMeshReductionManagerModule::GetDistributedMeshMergingInterface() const
{
	return DistributedMeshMerging;
}

void FMeshReductionManagerModule::OnModuleConfigurationCompleted(FSimpleDelegate Delegate)
{
	// If the configuration delegate was already called, execute the provided delegate right away
	if (bConfigurationCompleted)
	{
		Delegate.ExecuteIfBound();
	}
	else
	{
		ConfigurationCompletedDelegate.Add(Delegate);
	}
}