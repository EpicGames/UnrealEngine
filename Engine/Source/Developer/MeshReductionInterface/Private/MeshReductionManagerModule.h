// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMeshReductionManagerModule.h"

class IMeshReduction;
class IMeshMerging;

class FMeshReductionManagerModule : public IMeshReductionManagerModule
{
public:
	FMeshReductionManagerModule();

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	/** Returns the mesh reduction plugin if available. */
	virtual IMeshReduction* GetStaticMeshReductionInterface() const override;

	/** Returns the mesh reduction plugin if available. */
	virtual IMeshReduction* GetSkeletalMeshReductionInterface() const override;

	/** Returns the mesh merging plugin if available. */
	virtual IMeshMerging* GetMeshMergingInterface() const override;

	virtual IMeshMerging* GetDistributedMeshMergingInterface() const override;

	virtual void OnModuleConfigurationCompleted(FSimpleDelegate Delegate) override;

private:
	/** Cached pointer to the mesh reduction interface. */
	IMeshReduction* StaticMeshReduction;
	/** Cached pointer to the mesh reduction interface. */
	IMeshReduction* SkeletalMeshReduction;
	/** Cached pointer to the mesh merging interface. */
	IMeshMerging* MeshMerging;
	/** Cached pointer to the distributed mesh merging interface. */
	IMeshMerging* DistributedMeshMerging;

	bool bConfigurationCompleted = false;
	FSimpleMulticastDelegate ConfigurationCompletedDelegate;
};
