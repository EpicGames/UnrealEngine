// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "MeshMerge/MeshProxySettings.h"
#include "Modules/ModuleInterface.h"

struct FOverlappingCorners;
struct FMeshDescription;

/**
* Mesh reduction interface.
*/
class IMeshReduction
{
public:
	virtual ~IMeshReduction() = default;

	/**
	* Reduces the raw mesh using the provided reduction settings.
	* @param OutReducedMesh - Upon return contains the reduced mesh.
	* @param OutMaxDeviation - Upon return contains the maximum distance by which the reduced mesh deviates from the original.
	* @param InMesh - The mesh to reduce.
	* @param ReductionSettings - Setting with which to reduce the mesh.
	*/
	virtual void ReduceMeshDescription(
		FMeshDescription& OutReducedMesh,
		float& OutMaxDeviation,
		const FMeshDescription& InMesh,
		const FOverlappingCorners& InOverlappingCorners,
		const struct FMeshReductionSettings& ReductionSettings
	) = 0;

	/**
	* Reduces the provided skeletal mesh.
	* @returns true if reduction was successful.
	*/
	virtual bool ReduceSkeletalMesh(
		class USkeletalMesh* SkeletalMesh,
		int32 LODIndex,
		const class ITargetPlatform* TargetPlatform
	) = 0;
	/**
	* Returns a unique string identifying both the reduction plugin itself and the version of the plugin.
	*/
	virtual const FString& GetVersionString() const = 0;

	/**
	*	Returns true if mesh reduction is supported
	*/
	virtual bool IsSupported() const = 0;

	/**
	*	Returns true if mesh reduction is active. Active mean there will be a reduction of the vertices or triangle number
	*/
	virtual bool IsReductionActive(const struct FMeshReductionSettings &ReductionSettings) const = 0;
	virtual bool IsReductionActive(const struct FMeshReductionSettings& ReductionSettings, uint32 NumVertices, uint32 NumTriangles) const = 0;

	virtual bool IsReductionActive(const struct FSkeletalMeshOptimizationSettings &ReductionSettings) const = 0;
	virtual bool IsReductionActive(const struct FSkeletalMeshOptimizationSettings &ReductionSettings, uint32 NumVertices, uint32 NumTriangles) const = 0;
};

DECLARE_DELEGATE_ThreeParams(FProxyCompleteDelegate, struct FMeshDescription&, struct FFlattenMaterial&, const FGuid);
DECLARE_DELEGATE_TwoParams(FProxyFailedDelegate, const FGuid, const FString&);
DECLARE_DELEGATE_TwoParams(FCreateProxyDelegate, const FGuid, TArray<UObject*>&);

/** Data used for passing back the data resulting from a completed mesh merging operation*/ 
struct FMergeCompleteData
{
	/** Outer object to store/save UObjects */
	class UPackage* InOuter;
	/** Base package name for the proxy mesh UObjects */
	FString ProxyBasePackageName;
	/** Merge/Proxy settings structure */
	FMeshProxySettings InProxySettings;
	/** Callback delegate object used as a callback when the job finishes */
	FCreateProxyDelegate CallbackDelegate;
	/** List of Components containing imposter LODs */
	TArray<const UStaticMeshComponent*> ImposterComponents;
	/** List of Components incorporated into the Proxy Mesh */
	TArray<UStaticMeshComponent*> StaticMeshComponents;
	/** Base material to instance for the proxy material */
	UMaterialInterface* BaseMaterial;
};

struct FFlattenMaterial;

DECLARE_DELEGATE_OneParam(FBakeMaterialsDelegate, TArray<FFlattenMaterial>&);
/**
* Mesh merging interface.
*/
class IMeshMerging
{
public:
	virtual ~IMeshMerging() = default;

	virtual void ProxyLOD(const TArray<struct FMeshMergeData>& InData,
		const struct FMeshProxySettings& InProxySettings,
		const TArray<struct FFlattenMaterial>& InputMaterials,
		const FGuid InJobGUID) {}
	
	virtual void ProxyLOD(const TArray<struct FInstancedMeshMergeData>& InData,
		const struct FMeshProxySettings& InProxySettings,
		const TArray<struct FFlattenMaterial>& InputMaterials,
		const FGuid InJobGUID) {}

	virtual void AggregateLOD() {}

	virtual FString GetName() = 0;

	/*
	* Does this tool support baking materials in parallel
	*/
	virtual bool bSupportsParallelMaterialBake() { return false; }

	FProxyCompleteDelegate CompleteDelegate;
	FProxyFailedDelegate FailedDelegate;
	FBakeMaterialsDelegate BakeMaterialsDelegate;
};

/**
* Mesh reduction module interface.
*/
class IMeshReductionModule : public IModuleInterface, public IModularFeature
{
public:
	/**
	* Retrieve the static mesh reduction interface.
	*/
	virtual class IMeshReduction* GetStaticMeshReductionInterface() = 0;

	/**
	* Retrieve the static mesh reduction interface.
	*/
	virtual class IMeshReduction* GetSkeletalMeshReductionInterface() = 0;

	/**
	* Retrieve the mesh merging interface.
	*/
	virtual class IMeshMerging* GetMeshMergingInterface() = 0;

	/**
	* Retrieve the distributed mesh merging interface.
	*/
	virtual class IMeshMerging* GetDistributedMeshMergingInterface() = 0;

	virtual FString GetName() = 0;

	// Modular feature name to register for retrieval during runtime
	static const FName GetModularFeatureName()
	{
		return TEXT("MeshReduction");
	}
};
