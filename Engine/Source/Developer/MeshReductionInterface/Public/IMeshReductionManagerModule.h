// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Delegates/Delegate.h"

class IMeshReduction;
class IMeshMerging;

class IMeshReductionManagerModule : public IModuleInterface
{
public:	
	/** Returns the mesh reduction plugin if available. */
	virtual IMeshReduction* GetStaticMeshReductionInterface() const = 0;

	/** Returns the mesh reduction plugin if available. */
	virtual IMeshReduction* GetSkeletalMeshReductionInterface() const = 0;

	/** Returns the mesh merging plugin if available. */
	virtual IMeshMerging* GetMeshMergingInterface() const = 0;

	/** Returns the distributed mesh merging plugin if available. */
	virtual IMeshMerging* GetDistributedMeshMergingInterface() const = 0;

	/** Register a delegate that will fire when the module configuration is completed. */
	virtual void OnModuleConfigurationCompleted(FSimpleDelegate Delegate) = 0;
};
