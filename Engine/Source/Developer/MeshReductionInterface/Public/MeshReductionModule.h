// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class IMeshReduction;
class IMeshMerging;

class FMeshReductionModule : public IModuleInterface
{
public:
	FMeshReductionModule();

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	/** Returns the mesh reduction plugin if available. */
	virtual IMeshReduction* GetStaticMeshReductionInterface();

	/** Returns the mesh reduction plugin if available. */
	virtual IMeshReduction* GetSkeletalMeshReductionInterface();

	/** Returns the mesh merging plugin if available. */
	virtual IMeshMerging* GetMeshMergingInterface();

	virtual IMeshMerging* GetDistributedMeshMergingInterface();
private:
	/** Cached pointer to the mesh reduction interface. */
	IMeshReduction* StaticMeshReduction;
	/** Cached pointer to the mesh reduction interface. */
	IMeshReduction* SkeletalMeshReduction;
	/** Cached pointer to the mesh merging interface. */
	IMeshMerging* MeshMerging;
	/** Cached pointer to the distributed mesh merging interface. */
	IMeshMerging* DistributedMeshMerging;
};
